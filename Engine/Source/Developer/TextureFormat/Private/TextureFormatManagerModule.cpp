// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Modules/ModuleManager.h"
#include "TextureFormatManager.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatManager, Log, All);

/**
 * Module for the target platform manager
 */
class FTextureFormatManagerModule
	: public ITextureFormatManagerModule
{
public:

	enum class EInitPhase
	{
		JustConstructedNotInit = 0,
		Invalidated = 1,
		GetTextureFormatsInProgressDontTouch = 2,
		GetTextureFormatsPartialOkayToRead = 3, // values >= here are okay to make queries
		GetTextureFormatsDone = 4
	};

	FTextureFormatManagerModule()
		: ModuleName(TEXT("TextureFormat"))
		, bForceCacheUpdate(true)
		, TextureFormatsInitPhase(EInitPhase::JustConstructedNotInit)
	{
		UpdateTextureFormatList();

		// For tracking texture format discovery.
		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FTextureFormatManagerModule::ModulesChangesCallback);
	}

	/** Destructor. */
	virtual ~FTextureFormatManagerModule() = default;

	virtual void ShutdownModule()
	{
		FModuleManager::Get().OnModulesChanged().RemoveAll(this);
	}

	void UpdateTextureFormatList()
	{
		FScopeLock Lock(&ModuleMutex);

		// should not be called recursively while I am building the list :
		check( TextureFormatsInitPhase!= EInitPhase::GetTextureFormatsInProgressDontTouch );

		// bForceCacheUpdate should be true on first call, so we don't need a separate static init flag
		if ( bForceCacheUpdate )
		{
			// turn off flag immediately so that repeated calls to GetTextureFormats will not come in here again
			bForceCacheUpdate = false;
			TextureFormatsInitPhase = EInitPhase::GetTextureFormatsInProgressDontTouch;

			// note the first time this is done is from FTargetPlatformManagerModule::FTargetPlatformManagerModule()
			//	so calls to it are dangerous

			TextureFormats.Empty(TextureFormats.Num());
			TextureFormatMetadata.Empty(TextureFormatMetadata.Num());

			TArray<FName> Modules;

			FModuleManager::Get().FindModules(TEXT("*TextureFormat*"), Modules);

			if (!Modules.Num())
			{
				UE_LOGF(LogTextureFormatManager, Error, "No texture formats found!");
			}
			
			// This is all because child formats will do a LoadModule on base formats and expect them to be ready, so
			// we make sure they are done first.
			TArray<FTextureFormatMetadata> BaseModules;
			TArray<FTextureFormatMetadata> ChildModules;

			for (int32 Index = 0; Index < Modules.Num(); Index++)
			{
				if (Modules[Index] != ModuleName) // Avoid our own module when going through this list that was gathered by name
				{
					ITextureFormatModule* Module = FModuleManager::LoadModulePtr<ITextureFormatModule>(Modules[Index]);
					if (Module)
					{
						FTextureFormatMetadata ModuleMeta;
						ModuleMeta.Module = Module;
						ModuleMeta.ModuleName = Modules[Index];
						if (Module->CanCallGetTextureFormats()) // child modules want to call GetTextureFormats
						{
							ChildModules.Add(ModuleMeta);
						}
						else
						{
							BaseModules.Add(ModuleMeta);
						}
					}
				}
			}
			
			// first populate TextureFormats[] with all Base Modules
			for (int32 Index = 0; Index < BaseModules.Num(); Index++)
			{
				ITextureFormatModule* Module = BaseModules[Index].Module;

				ITextureFormat* Format = Module->GetTextureFormat();
				if (Format != nullptr)
				{
					// I want to see this log by default in Cook+Editor , but not in TBW
					#ifndef VerboseIfNotEditor
					#if WITH_EDITOR
					#define VerboseIfNotEditor	Display
					#else
					#define VerboseIfNotEditor	Verbose
					#endif
					#endif

					UE_LOGF(LogTextureFormatManager, VerboseIfNotEditor,"Loaded Base TextureFormat: %ls",*BaseModules[Index].ModuleName.ToString());
						
					TextureFormats.Add(Format);
					TextureFormatMetadata.Add(BaseModules[Index]);
				}
			}
			
			// Init phase 3 means you are now allowd to call GetTextureFormats() and you will get only the Base formats
			TextureFormatsInitPhase = EInitPhase::GetTextureFormatsPartialOkayToRead;

			// run through the Child formats and call GetTextureFormat() on them
			// this could call back to me and do GetTextureFormats() which will get only the base formats
			TArray<TPair<ITextureFormat*, int32>, TInlineAllocator<32>> ReadyChildModules;
			for (int32 Index = 0; Index < ChildModules.Num(); Index++)
			{
				ITextureFormatModule* Module = ChildModules[Index].Module;

				ITextureFormat* Format = Module->GetTextureFormat();
				if (Format != nullptr)
				{
					UE_LOGF(LogTextureFormatManager,VerboseIfNotEditor,"Loaded Child TextureFormat: %ls",*ChildModules[Index].ModuleName.ToString());

					// do not add me to TextureFormats yet'
					ReadyChildModules.Add({Format, Index});
				}
			}

			for (TPair<ITextureFormat*, int32>& ReadyChild : ReadyChildModules)
			{
				TextureFormats.Add(ReadyChild.Key);
				TextureFormatMetadata.Add(ChildModules[ReadyChild.Value]);
			}
			
			// all done :
			TextureFormatsInitPhase = EInitPhase::GetTextureFormatsDone;
		}

		check( (int)TextureFormatsInitPhase >= (int)EInitPhase::GetTextureFormatsPartialOkayToRead );
	}
	
	virtual TArray<const class ITextureFormat*> GetAllTextureFormats() override
	{
		// note : does not call UpdateTextureFormatList , similar to FindTextureFormat
		//	assumes list is kept up to date

		TArray<const class ITextureFormat*> Ret;
		
		// copy member variable to local with lock held
		{
		FScopeLock Lock(&ModuleMutex);
		Ret = TextureFormats;
		}
		
		return MoveTemp(Ret);
	}

	virtual const ITextureFormat* FindTextureFormat(FName Name) override
	{
		// just pass through to FindTextureFormatAndModule
		FName ModuleNameUnused;
		ITextureFormatModule* ModuleUnused;
		return FindTextureFormatAndModule(Name, ModuleNameUnused, ModuleUnused);
	}
	
	virtual const class ITextureFormat* FindTextureFormatAndModule(FName Name, FName& OutModuleName, ITextureFormatModule*& OutModule) override
	{
		FScopeLock Lock(&ModuleMutex);
		check( (int)TextureFormatsInitPhase >= (int)EInitPhase::GetTextureFormatsPartialOkayToRead );

		check( ! bForceCacheUpdate );

		for (int32 Index = 0; Index < TextureFormats.Num(); Index++)
		{
			TArray<FName> Formats;

			TextureFormats[Index]->GetSupportedFormats(Formats);

			for (int32 FormatIndex = 0; FormatIndex < Formats.Num(); FormatIndex++)
			{
				if (Formats[FormatIndex] == Name)
				{
					const FTextureFormatMetadata& FoundMeta = TextureFormatMetadata[Index];
					OutModuleName = FoundMeta.ModuleName;
					OutModule = FoundMeta.Module;
					return TextureFormats[Index];
				}
			}
		}

		return nullptr;
	}


private:

	void ModulesChangesCallback(FName InModuleName, EModuleChangeReason ReasonForChange)
	{
		//
		// This is complex because this is the only place we can set up our texture format list
		// from the game thread. The only time we can update "on demand" could be from any thread,
		// which prevents up from calling LoadModule.
		//
		// However, this gets called while we are loading our modules.
		//
		// In order to avoid recursion, we only do LoadModules in response to module _discovery_, 
		// thus ensuring we can't recurse from our Invalidate call.
		//
		//
		if (ReasonForChange != EModuleChangeReason::PluginDirectoryChanged || // only care about discovery
			(InModuleName == ModuleName) ||  // don't care about _this_ module
			!InModuleName.ToString().Contains(TEXT("TextureFormat"))) // only care about texture formats.
		{
			return;
		}

		// when a "TextureFormat" module is discovered, rebuild my list.

		// Note... it's unclear but it looks like it _might_ be possible for a LoadModule to 
		// cause a module that it loads to get discovered, which could cause a recursion. However
		// texture format modules are pretty straightforward and shouldn't ever get here as they
		// are discovered on startup.

		// In order to even get here you have to put a texture format in a plugin that gets loaded after
		// startup. (I think)
		UpdateTextureFormatList();
	}

	const FName ModuleName;

	TArray<const ITextureFormat*> TextureFormats;

	struct FTextureFormatMetadata
	{
		FName ModuleName;
		ITextureFormatModule* Module;
	};
	TArray<FTextureFormatMetadata> TextureFormatMetadata;

	// Flag to force reinitialization of all cached data. This is needed to have up-to-date caches
	// in case of a module reload of a TextureFormat-Module.
	bool bForceCacheUpdate;

	// Track tricky initialization progress
	EInitPhase TextureFormatsInitPhase;

	FCriticalSection ModuleMutex;
};

IMPLEMENT_MODULE(FTextureFormatManagerModule, TextureFormat);
