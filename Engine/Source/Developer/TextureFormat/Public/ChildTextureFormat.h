// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "TextureCompressorModule.h"
#include "Algo/Unique.h"
#include "TextureFormatManager.h"

/**
 * Version of ITextureFormat that handles a child texture format that is used as a "post-process" after compressing textures, useful for
 * several platforms that need to modify already compressed texture data for optimal data.
 * 
 * There are 3 ways a texture can get processed:
 *		-- ITextureFormat::SupportsTiling returns true. This should only be for the old Oodle RAD Game Tools supplied texture encoder 
 *			and is being deprecated in favor of the new built-in Oodle texture encoder (TextureFormatOodle). In this case,
 *			the child format passes the images to the encoder to split apart prior to passing to the platform tools.
 *		-- FChildTextureFormat::GetTiler returns nullptr. This is being phased out because the linear version of the input texture
 *			gets encoded for each platform with a post process, wasting cook time. In this case the child format encodes
 *			the "base" texture and then tiles it using the platform tools.
 *		-- FChildTextureFormat::GetTiler returns a valid pointer. This is the most recent design. In this case, the child format 
 *			is used as a holder of metadata and doesn't actually do any work (CompressImage class of functions not called).
 */
class FChildTextureFormat : public ITextureFormat
{
public:

	FChildTextureFormat(const TCHAR* PlatformFormatPrefix)
		: FormatPrefix(PlatformFormatPrefix)
	{

	}

	virtual const FChildTextureFormat* GetChildFormat() const override final
	{
		return this;
	}

	/**
	*	Return the texture tiler for this platform. If present, then the texture build process
	*	will try to use the cached linear encoding for the texture as input rather than rebuilding
	*	the linear texture prior to tiling. Confusingly, this can not happen if the base texture
	*	format returns true for SupportsTiling, as that refers to the texture encoder, not the format.
	*/
	virtual const ITextureTiler* GetTiler() const
	{
		return nullptr;
	}

	FName GetBaseFormatName(FName ChildFormatName) const
	{
		return FName(*(ChildFormatName.ToString().Replace(*FormatPrefix, TEXT(""))));
	}

	/**
	 * Given a platform specific format name, get the texture format object that will do the encoding.
	 */
	const ITextureFormat* GetBaseFormatObject(FName FormatName) const
	{
		FName BaseFormatName = GetBaseFormatName(FormatName);

		ITextureFormatManagerModule& TFM = GetTextureFormatManagerRef();
		const ITextureFormat* FormatObject = TFM.FindTextureFormat(BaseFormatName);

		checkf(FormatObject != nullptr, TEXT("Bad FormatName %s passed to FChildTextureFormat::GetBaseFormatObject()"), *BaseFormatName.ToString());

		return FormatObject;
	}


protected:

	void AddBaseTextureFormatModules(const TCHAR* ModuleNameWildcard)
	{
		TArray<FName> Modules;
		FModuleManager::Get().FindModules(ModuleNameWildcard, Modules);

		for (FName ModuleName : Modules)
		{
			ITextureFormatModule * TFModule = FModuleManager::LoadModulePtr<ITextureFormatModule>(ModuleName);
			if ( TFModule != nullptr )
			{
				ITextureFormat* BaseFormat = TFModule->GetTextureFormat();
				if ( BaseFormat != nullptr )
				{
				BaseFormat->GetSupportedFormats(BaseFormats);
			}
		}
	}
	}
	
	void FinishAddBaseTextureFormatModules() 
	{
		// make unique:		
		BaseFormats.Sort( FNameFastLess() );
		BaseFormats.SetNum( Algo::Unique( BaseFormats ) );
		
		check( SupportedFormatsCached.IsEmpty() );
		SupportedFormatsCached.Empty(BaseFormats.Num());
		for (FName BaseFormat : BaseFormats)
		{
			FName ChildFormat(*(FormatPrefix + BaseFormat.ToString()));
			SupportedFormatsCached.Add(ChildFormat);
		}
	}


	FCbObjectView GetBaseFormatConfigOverride(const FCbObjectView& ObjView) const
	{
		return ObjView.FindView("BaseTextureFormatConfig").AsObjectView();
	}
public:
	FTextureBuildSettings GetBaseTextureBuildSettings(const FTextureBuildSettings& BuildSettings) const
	{
		FTextureBuildSettings BaseSettings = BuildSettings;
		BaseSettings.TextureFormatName = GetBaseFormatName(BuildSettings.TextureFormatName);
		BaseSettings.FormatConfigOverride = GetBaseFormatConfigOverride(BuildSettings.FormatConfigOverride);
		return BaseSettings;
	}
protected:

	/**
	 * The final version is a combination of parent and child formats, 8 bits for each
	 */
	virtual uint8 GetChildFormatVersion(FName Format, const FTextureBuildSettings* BuildSettings) const = 0;

	/**
	 * Make the child type think about if they need a key string or not, by making it pure virtual.
	 * InMipCount and InMip0Dimensions are only valid for non-virtual textures (VTs should never call this function as they never tile)
	 */
	virtual FString GetChildDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const = 0;

	/**
	 * Obtains the global format config object for this texture format.
	 * 
	 * @param BuildSettings Build settings.
	 * @returns The global format config object or an empty object if no format settings are defined for this texture format.
	 */
	virtual FCbObject ExportGlobalChildFormatConfig(const FTextureBuildSettings& BuildSettings) const
	{
		return FCbObject();
	}

	/**
	 * Obtains the format config appropriate for the build .
	 * 
	 * @param ObjView A view of the entire format config container or null if none exists.
	 * @returns The format settings object view or a null view if the active global format config should be used.
	 */
	virtual FCbObjectView GetChildFormatConfigOverride(const FCbObjectView& ObjView) const
	{
		return ObjView.FindView("ChildTextureFormatConfig").AsObjectView();
	}

public:

	//// ITextureFormat interface ////

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Append(SupportedFormatsCached);
	}

	virtual bool SupportsEncodeSpeed(FName Format, const ITargetPlatformSettings* TargetPlatform) const override
	{
		return GetBaseFormatObject(Format)->SupportsEncodeSpeed(Format, TargetPlatform);
	}

	virtual bool CanAcceptNonF32Source(FName Format) const override
	{
		return GetBaseFormatObject(Format)->CanAcceptNonF32Source(Format);
	}

	virtual FName GetEncoderName(FName Format) const override
	{
		return GetBaseFormatObject(Format)->GetEncoderName(Format);
	}

	virtual uint16 GetVersion(FName Format, const FTextureBuildSettings* BuildSettings) const final
	{
		uint16 BaseVersion = GetBaseFormatObject(Format)->GetVersion(Format, BuildSettings);
		checkf(BaseVersion < 256, TEXT("BaseFormat for %s had too large a version (%d), must fit in 8bits"), *Format.ToString(), BaseVersion);

		uint8 ChildVersion = GetChildFormatVersion(Format, BuildSettings);

		// 8 bits for each version
		return (uint16)((BaseVersion << 8) | ChildVersion);
	}

	virtual FString GetDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const final
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(InBuildSettings);

		FString BaseString = GetBaseFormatObject(InBuildSettings.TextureFormatName)->GetDerivedDataKeyString(BaseSettings, InMipCount, InMip0Dimensions);
		FString ChildString = GetChildDerivedDataKeyString(InBuildSettings, InMipCount, InMip0Dimensions);

		return BaseString + ChildString;
	}

	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& InBuildSettings, bool bImageHasAlphaChannel) const
	{
		FTextureBuildSettings Settings = GetBaseTextureBuildSettings(InBuildSettings);
		return GetBaseFormatObject(InBuildSettings.TextureFormatName)->GetEncodedPixelFormat(Settings, bImageHasAlphaChannel);
	}

	bool CompressBaseImage(
		const FImage& InImage,
		const FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions, 
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		FCompressedImage2D& OutCompressedImage
	) const
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		// pass along the compression to the base format
		if (GetBaseFormatObject(BuildSettings.TextureFormatName)->CompressImage(InImage, BaseSettings, InMip0Dimensions, InMip0NumSlicesNoDepth, InMipIndex, InMipCount, DebugTexturePathName, bImageHasAlphaChannel, OutCompressedImage) == false)
		{
			UE_LOGF(LogTemp, Error, "Failed to compress with base compressor [format %ls]", *BaseSettings.TextureFormatName.ToString());
			return false;
		}
		return true;
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool CompressBaseImageTiled(
		const FImage* Images,
		uint32 NumImages,
		const FTextureBuildSettings& BuildSettings,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		TSharedPtr<FTilerSettings>& TilerSettings,
		FCompressedImage2D& OutCompressedImage
	) const
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		// pass along the compression to the base format
		if (GetBaseFormatObject(BuildSettings.TextureFormatName)->CompressImageTiled(Images, NumImages, BaseSettings, DebugTexturePathName, bImageHasAlphaChannel, TilerSettings, OutCompressedImage) == false)
		{
			UE_LOGF(LogTemp, Error, "Failed to compress with base tiled compressor [format %ls]", *BaseSettings.TextureFormatName.ToString());
			return false;
		}
		return true;
	}

	bool PrepareTiling(
		const FImage* Images,
		const uint32 NumImages,
		const FTextureBuildSettings& BuildSettings,
		bool bImageHasAlphaChannel,
		TSharedPtr<FTilerSettings>& OutTilerSettings,
		TArray<FCompressedImage2D>& OutCompressedImages
	) const override
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		return GetBaseFormatObject(BuildSettings.TextureFormatName)->PrepareTiling(Images, NumImages, BaseSettings, bImageHasAlphaChannel, OutTilerSettings, OutCompressedImages);
	}

	bool SetTiling(
		const FTextureBuildSettings& BuildSettings,
		TSharedPtr<FTilerSettings>& TilerSettings,
		const TArray64<uint8>& ReorderedBlocks,
		uint32 NumBlocks
	) const override
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		return GetBaseFormatObject(BuildSettings.TextureFormatName)->SetTiling(BaseSettings, TilerSettings, ReorderedBlocks, NumBlocks);
	}

	void ReleaseTiling(const FTextureBuildSettings& BuildSettings, TSharedPtr<FTilerSettings>& TilerSettings) const override
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		return GetBaseFormatObject(BuildSettings.TextureFormatName)->ReleaseTiling(BuildSettings, TilerSettings);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS


	virtual FCbObject ExportGlobalFormatConfig(const FTextureBuildSettings& BuildSettings) const override
	{
		FTextureBuildSettings BaseSettings = GetBaseTextureBuildSettings(BuildSettings);

		FCbObject BaseObj = GetBaseFormatObject(BuildSettings.TextureFormatName)->ExportGlobalFormatConfig(BaseSettings);
		FCbObject ChildObj = ExportGlobalChildFormatConfig(BuildSettings);

		if (!BaseObj && !ChildObj)
		{
			return FCbObject();
		}

		FCbWriter Writer;
		Writer.BeginObject("TextureFormatConfig");

		if (BaseObj)
		{
			Writer.AddObject("BaseTextureFormatConfig", BaseObj);
		}

		if (ChildObj)
		{
			Writer.AddObject("ChildTextureFormatConfig", ChildObj);
		}

		Writer.EndObject();

		return Writer.Save().AsObject();
	}

protected:

	// Prefix put before all formats from parent formats
	const FString FormatPrefix;

	// List of base formats that. Combined with FormatPrefix, this contains all formats this can handle
	TArray<FName> BaseFormats;

	// List of combined BaseFormats with FormatPrefix.
	TArray<FName> SupportedFormatsCached;
};
