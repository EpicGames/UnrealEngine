// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "Serialization/CompactBinary.h"

class ITargetPlatformSettings;
struct FTextureBuildSettings;
class FChildTextureFormat;

/**
 * Structure for texture format compressor capabilities.
 * This struct is deprecated - FEncodedTextureExtendedData is used instead.
 */
struct FTextureFormatCompressorCaps
{
	FTextureFormatCompressorCaps()
		: MaxTextureDimension_DEPRECATED(TNumericLimits<uint32>::Max())
		, NumMipsInTail_DEPRECATED(0)
		, ExtData_DEPRECATED(0)
	{ }

	// MaxTextureDimension is never set, remove it
	uint32 MaxTextureDimension_DEPRECATED;
	uint32 NumMipsInTail_DEPRECATED;
	uint32 ExtData_DEPRECATED;
};

/**
* Holds various engine configuration parameters that can affect the output of a build but
* should generally be constant across all texture builds. These are sourced from CVars
* and enums/defines that aren't necessarily visible in all modules.
* 
* This structure serializes to compact binary only writing if the values are not default,
* so changing the default initialization without changing the texture build version/guid
* can result in build mismatch.
* 
* Created via GenerateTextureEngineParameters() in TextureDerivedDataTask.cpp.
*/
struct FTextureEngineParameters
{
	bool bEngineSupportsVolumeTextureStreaming = true;			// GEngineSupportsVolumeTextureStreaming
	bool bEngineSupportsTexture2DArrayStreaming = true;			// GEngineSupportsTexture2DArrayStreaming
	int32 NumInlineDerivedMips = 7;								// NUM_INLINE_DERIVED_MIPS
};

static bool GetStreamingDisabledForNonVirtualTextureProperties(bool bInCubeMap, bool bInVolumeTexture, bool bInTextureArray, const FTextureEngineParameters& InEngineParameters)
{
	if (bInCubeMap)
	{
		return true;
	}
	if (bInVolumeTexture && InEngineParameters.bEngineSupportsVolumeTextureStreaming == false)
	{
		return true;
	}
	if (bInTextureArray && InEngineParameters.bEngineSupportsTexture2DArrayStreaming == false)
	{
		return true;
	}
	return false;
}



// Extra data for an encoded texture.
// This is filled out by platform "child" formats that can potentially tile or otherwise
// reorganize the mip data for use on the target platform. For platforms that use bog standard
// "linear" texture data, this is mostly zeroes, but must still be "Valid" data returned by the
// GetExtendedDataForTexture - notably the MipSizesInBytes.
struct FEncodedTextureExtendedData
{
	// Copied from RHIDefinitions.h - we expose here to avoid the include.
	static constexpr int32 MAX_TEXTURE_MIP_COUNT=15;

	// Some platforms group the smallest mips in to a single bulk chunk of data. This is nonzero
	// in such cases and the MipSizesInBytes array is truncated.
	int32 NumMipsInTail = 0;

	// ExtData is metadata the tiler passes through to the runtime.
	uint32 ExtData = 0;
	
	// If true, the mip data can't be assumed to be linear.
	bool bIsTiled = false;

	// If true, this texture might change layouts if top mips are striped (i.e. LODBias is not zero).
	bool bSensitiveToLODBias = false;
	
	// If bSensitiveToLODBias is set, this is the LODBias for this layout.
	int8 LODBiasIfSensitive = 0;

	// All generators of this structure are required to fill this structure out even if they are
	// default linear sizes. For non linear textures these are not trivially calculatable.
	// Also note that if the texture is sensitive to LODBias with respect to tiling, the indeces in this
	// array that are for mips that would be stripped are present but should be ignored as they don't 
	// represent anything we can accurately calculate.
	TArray<uint64, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> MipSizesInBytes;
};


/**
* Calculate the number of streaming mips for the given set of texture properties. This must work off
* of properties that can (eventually) be calculated without running a full texture build.
* 
* Texture mips are split in to two large groups: streaming and non-streaming (aka "inline"). Note
* that "inline" is sometimes used as a verb to mean "load off of disk and place in our bulk data".
* "Inline" textures are loaded with the texture asset, and streaming textures are loaded on demand.
* Generally, 7 of the smallest mips are inlined, however some platforms pack a lot of mips in to a single
* allocation ("packed mip tail" = NumMipsInTail). Those mips must all be inlined.
*
* This function is used primarily for determining where to put mips in the DDC. Runtime inlining is subject
* to further constraints and must use the GetNumNonStreamingMipsDirect() function.
* 
* InNumMips			    The total mips that the texture contains.
* InExtendedData	    If the texture is being built for a platform that provides extended data, pass it
*					    here. For platforms that don't need it (i.e. PC), this should be nullptr.
* InEngineParameters    Holds a vairety of engine configuration constants, create with GenerateEngineParameters()
*
*/
static int32 GetNumStreamingMipsDirect(int32 InNumMips, bool bInCubeMap, bool bInVolumeTexture, bool bInTextureArray, const FEncodedTextureExtendedData* InExtendedData, const FTextureEngineParameters& InEngineParameters)
{
	bool bAllowStreaming = true;
	{
		const bool bDisableStreaming = GetStreamingDisabledForNonVirtualTextureProperties(bInCubeMap, bInVolumeTexture, bInTextureArray, InEngineParameters);
		if (bDisableStreaming)
			bAllowStreaming = false;
	}

	int32 NumStreamingMips = 0;
	if (bAllowStreaming)
	{
		// Some platforms pack several mips in to a single entry. If this is the case,
		// those must be non-streaming.
		int32 NumMipsInTail = 0;
		if (InExtendedData)
		{
			NumMipsInTail = InExtendedData->NumMipsInTail;
		}

		int32 NumInlineMips = FMath::Max(NumMipsInTail, InEngineParameters.NumInlineDerivedMips);
		NumStreamingMips = FMath::Max(0, InNumMips - NumInlineMips);
	}
	return NumStreamingMips;
}

/**
	Return how many mips need to be inlined for the given texture properties. This pays attention to runtime
	size restrictions and should be used for serializing cooked data.

	InNumMips, InTopMipSizeX, InTopMipSizeY must all be sizes _before_ any LOD stripping during cook, with the expectation
	that the returned value is verified against the remaining mip count after LOD stripping.

	This function is not valid if streaming is disabled for the texture.
*/
static int32 GetNumNonStreamingMipsDirect(int32 InNumMips, int32 InTopMipSizeX, int32 InTopMipSizeY, EPixelFormat InPixelFormat, int32 InNumMipsInTail, int32 InNumInlineDerivedMips)
{
	if (InNumMips <= 1)
	{
		return InNumMips;
	}

	int32 NumNonStreamingMips = 1;

	// Take the min resident limit into account.
	if (InNumMipsInTail)
	{
		// If we have a tail, it can't be broken up so the whole thing must be non streaming.
		NumNonStreamingMips = FMath::Max(NumNonStreamingMips, InNumMipsInTail);
	}
	NumNonStreamingMips = FMath::Max(NumNonStreamingMips, InNumInlineDerivedMips);
	NumNonStreamingMips = FMath::Min(NumNonStreamingMips, InNumMips);

	if ( RequiresBlock4Alignment(InPixelFormat) )
	{
		// ensure the top non-streamed mip (and all streamed mips) size is >= BlockSize (and a multiple of block size!)
			
		// note: this is not right for non pow 2; NeverStream should set !bIsStreamingPossible in that case
		if ( FMath::IsPowerOfTwo(InTopMipSizeX) && FMath::IsPowerOfTwo(InTopMipSizeY) )
		{
			// This is not right for textures smaller than 4x4, however such textures are already non-streaming due to
			// NumInlineDerivedMips (which is functionally hard coded to 7).
			NumNonStreamingMips = FMath::Max<int32>(NumNonStreamingMips, InNumMips - FPlatformMath::FloorLog2(InTopMipSizeX / 4));
			NumNonStreamingMips = FMath::Max<int32>(NumNonStreamingMips, InNumMips - FPlatformMath::FloorLog2(InTopMipSizeY / 4));
		}
		else
		{
			// should never get here as streaming should be disabled in this case, so we should not hit this branch, but it's not reliable
			NumNonStreamingMips = InNumMips;
		}
	}

	return NumNonStreamingMips;
}


// Everything necessary to know the memory layout for an encoded untiled unpacked texture (i.e. enough information
// to describe the texture entirely to a PC hardware API).
// Once a texture gets tiled or gets a packed mip tail, FEncodedTextureEncodedData is additionally
// required to know the memory layout.
struct FEncodedTextureDescription
{	
	int32 TopMipSizeX;
	int32 TopMipSizeY;
	int32 TopMipVolumeSizeZ; // This is 1 if bVolumeTexture == false
	int32 ArraySlices; // This is 1 if bTextureArray == false (including cubemaps)
	EPixelFormat PixelFormat;
	uint8 NumMips;
	bool bCubeMap;
	bool bTextureArray;
	bool bVolumeTexture;
	
	typedef TArray<FSharedBuffer, TInlineAllocator<FEncodedTextureExtendedData::MAX_TEXTURE_MIP_COUNT>> FSharedBufferMipChain;
	typedef TArray<FUniqueBuffer, TInlineAllocator<FEncodedTextureExtendedData::MAX_TEXTURE_MIP_COUNT>> FUniqueBufferMipChain;

	bool operator==(const FEncodedTextureDescription& OtherTextureDescription) const
	{
		return TopMipSizeX == OtherTextureDescription.TopMipSizeX &&
			TopMipSizeY == OtherTextureDescription.TopMipSizeY &&
			TopMipVolumeSizeZ == OtherTextureDescription.TopMipVolumeSizeZ &&
			ArraySlices == OtherTextureDescription.ArraySlices &&
			PixelFormat == OtherTextureDescription.PixelFormat &&
			NumMips == OtherTextureDescription.NumMips &&
			bCubeMap == OtherTextureDescription.bCubeMap &&
			bTextureArray == OtherTextureDescription.bTextureArray &&
			bVolumeTexture == OtherTextureDescription.bVolumeTexture;
	}

	// This returns the SizeZ value that is expected by RHI streamable texture structures. It
	// is only used by non-cube texture arrays and volumes, however the cubemap array and cubemap
	// values need to be consistent as they are persisted and would cause a DDC determinism issue.
	// (note that cubemap arrays are handled in the bTextureArray path).
	int32 GetRHIStyleSizeZ(int32 InMipIndex) const
	{
		return (bVolumeTexture || bTextureArray) ? GetNumSlices_WithDepth(InMipIndex) : 1;
	}

	// Returns the slice count for usage cases/platform that expect slice count to include
	// volume texture depth. InMipIndex only affects volume textures.
	int32 GetNumSlices_WithDepth(int32 InMipIndex) const
	{
		if (bVolumeTexture)
		{
			check(bTextureArray == false && bCubeMap == false);
			check(InMipIndex < NumMips);
			return FMath::Max(TopMipVolumeSizeZ >> InMipIndex, 1);
		}

		check ((bTextureArray && ArraySlices >= 1) || (!bTextureArray && ArraySlices == 1));
		int32 Slices = ArraySlices;
		if (bCubeMap)
		{
			Slices *= 6;
		}
		return Slices;
	}

	// Returns the slice count for usage cases/platforms that expect slice count to only include
	// cubemap/array slices.
	int32 GetNumSlices_NoDepth() const
	{
		if (bVolumeTexture)
		{
			check(bTextureArray == false && bCubeMap == false);
			return 1; // no such thing as a cube volume, or a volume array.
		}

		check((bTextureArray && ArraySlices >= 1) || (!bTextureArray && ArraySlices == 1));
		int32 Slices = ArraySlices;
		if (bCubeMap)
		{
			Slices *= 6;
		}
		return Slices;
	}

	int32 GetMipWidth(int32 InMipIndex) const
	{
		return FMath::Max(TopMipSizeX >> InMipIndex, 1);
	}
	int32 GetMipHeight(int32 InMipIndex) const
	{
		return FMath::Max(TopMipSizeY >> InMipIndex, 1);
	}
	// Always 1 unless volume texture.
	int32 GetMipDepth(int32 InMipIndex) const
	{
		return bVolumeTexture ? FMath::Max(TopMipVolumeSizeZ >> InMipIndex, 1) : 1;
	}
	static int32 GetMipWidth(int32 InTextureWidth, int32 InMipIndex)
	{
		return FMath::Max(InTextureWidth >> InMipIndex, 1);
	}
	static int32 GetMipHeight(int32 InTextureHeight, int32 InMipIndex)
	{
		return FMath::Max(InTextureHeight >> InMipIndex, 1);
	}
	static int32 GetMipDepth(int32 InTextureDepth, int32 InMipIndex, bool bInVolumeTexture)
	{
		return bInVolumeTexture ? FMath::Max(InTextureDepth >> InMipIndex, 1) : 1;
	}


	// Returns the size of the mip at the given index. Z is 1 unless it's a volume texture.
	FIntVector3 GetMipDimensions(int32 InMipIndex) const
	{
		FIntVector3 Results;
		Results.X = GetMipWidth(InMipIndex);
		Results.Y = GetMipHeight(InMipIndex);
		Results.Z = GetMipDepth(InMipIndex);
		return Results;
	}

	// Returns the byte size of the unpacked/tiled mip. For mip chains that are packed or tiled, use FEncodedTextureExtendedData::MipSizesInBytes.
	uint64 GetMipSizeInBytes(int32 InMipIndex) const
	{
		FIntVector3 MipDims = GetMipDimensions(InMipIndex);
		uint64 SliceByteCount = GPixelFormats[PixelFormat].Get2DImageSizeInBytes(MipDims.X, MipDims.Y);
		return SliceByteCount * GetNumSlices_WithDepth(InMipIndex);
	}

	// As GetMipSizeInBytes, except for a single slice of the mip.
	uint64 GetMipSliceSizeInBytes(int32 InMipIndex) const
	{
		FIntVector3 MipDims = GetMipDimensions(InMipIndex);
		return GPixelFormats[PixelFormat].Get2DImageSizeInBytes(MipDims.X, MipDims.Y);
	}

	// Returns the bytes necessary to get to the next row of the current mip
	uint64 GetMipSliceRowPitchBytes(int32 InMipIndex) const
	{
		const uint64 WidthInBlocks = GPixelFormats[PixelFormat].GetBlockCountForWidth(GetMipWidth(InMipIndex));
		return WidthInBlocks * GPixelFormats[PixelFormat].BlockBytes;
	}

	// This is the number of streaming mips without any restrictions associated with block size alignment. It's used
	// primarily for where mips go in the DDC.
	int32 GetNumStreamingMips(const FEncodedTextureExtendedData* InExtendedData, const FTextureEngineParameters& InEngineParameters) const
	{
		return GetNumStreamingMipsDirect(NumMips, bCubeMap, bVolumeTexture, bTextureArray, InExtendedData, InEngineParameters);
	}

	// This is the number of nonstreaming mips that works for cooking and thus is aware of block alignment concerns for runtime use.
	// Only call this if streaming is possible.
	int32 GetNumNonStreamingMips(const FEncodedTextureExtendedData* InExtendedData, const FTextureEngineParameters& InEngineParameters)
	{
		return GetNumNonStreamingMipsDirect(NumMips, TopMipSizeX, TopMipSizeY, PixelFormat, InExtendedData ? InExtendedData->NumMipsInTail : 0, InEngineParameters.NumInlineDerivedMips);
	}

	// Fills out the given array with linear mip sizes for the number of mips this texture has. These sizes are only valid
	// if the texture is linear and has no packed mip tail (i.e. no associated FEncodedTextureExtendedData).
	void GetLinearMipSizesInBytes(TArray<uint64, TInlineAllocator<FEncodedTextureExtendedData::MAX_TEXTURE_MIP_COUNT>>& OutMipSizesInBytes) const
	{
		OutMipSizesInBytes.SetNumUninitialized(NumMips);
		for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
		{
			OutMipSizesInBytes[MipIndex] = GetMipSizeInBytes(MipIndex);
		}
	}

	// Convenience function for iterating over the encoded mips when you need to know how many mips are represented. Use as:
	//
	//	for (int32 EncodedMipIndex = 0; EncodedMipIndex < OutMipTailIndex + 1; EncodedMipIndex++)
	//	{
	//		int32 MipsRepresentedThisIndex = EncodedMipIndex == OutMipTailIndex ? OutMipsInTail : 1;
	//	}
	//
	// This handles mip chains whether or not they have packed mip tails.
	// Note GetNumEncodedMips() == OutMipTailIndex + 1
	//
	void GetEncodedMipIterators(const FEncodedTextureExtendedData* InExtendedData, int32& OutMipTailIndex, int32& OutMipsInTail) const
	{
		OutMipTailIndex = NumMips - 1;
		OutMipsInTail = 1;
		if (InExtendedData && InExtendedData->NumMipsInTail > 1)
		{
			OutMipsInTail = InExtendedData->NumMipsInTail;
			OutMipTailIndex = NumMips - OutMipsInTail;
		}
	}

	// Returns the number of mips that actually carry bulk data for this texture. Nominally the number of total mips,
	// however some platforms have packed mip tails, which means they still have the total number of mips, but the last
	// several are all bundled together for memory savings.
	int32 GetNumEncodedMips(const FEncodedTextureExtendedData* InExtendedData) const
	{
		if (InExtendedData &&
			InExtendedData->NumMipsInTail > 1)
		{
			return NumMips - InExtendedData->NumMipsInTail + 1;
		}
		return NumMips;
	}

	// Returns the description _for the single mip level_ (i.e. no further mips)
	FEncodedTextureDescription GetDescriptionForMipLevel(const FEncodedTextureExtendedData* InExtendedData, int32 InMipIndex) const
	{
		check(InMipIndex < NumMips);

		FEncodedTextureDescription MipTextureDescription = *this;
		FIntVector3 TailFirstMipDims = GetMipDimensions(InMipIndex);
		MipTextureDescription.TopMipSizeX = TailFirstMipDims.X;
		MipTextureDescription.TopMipSizeY = TailFirstMipDims.Y;
		MipTextureDescription.TopMipVolumeSizeZ = TailFirstMipDims.Z;
		MipTextureDescription.NumMips = 1;
		if (InExtendedData && InExtendedData->NumMipsInTail && InMipIndex >= NumMips - InExtendedData->NumMipsInTail)
		{
			// we must only ever get the first mip tail index!
			check(InMipIndex == NumMips - InExtendedData->NumMipsInTail);

			// We want the layout for the entire tail.
			MipTextureDescription.NumMips = IntCastChecked<uint8>(InExtendedData->NumMipsInTail);
		}		
		return MipTextureDescription;
	}

	FEncodedTextureDescription RemoveTopMips(const FEncodedTextureExtendedData* InExtendedData, int32 InRemoveCount) const
	{
		check(InRemoveCount < NumMips);

		FEncodedTextureDescription MipTextureDescription = *this;
		FIntVector3 TailFirstMipDims = GetMipDimensions(InRemoveCount);
		MipTextureDescription.TopMipSizeX = TailFirstMipDims.X;
		MipTextureDescription.TopMipSizeY = TailFirstMipDims.Y;
		MipTextureDescription.TopMipVolumeSizeZ = TailFirstMipDims.Z;
		MipTextureDescription.NumMips = NumMips - static_cast<uint8>(InRemoveCount);
		if (InExtendedData && InExtendedData->NumMipsInTail && InRemoveCount >= NumMips - InExtendedData->NumMipsInTail)
		{
			// we must only ever get the first mip tail index!
			check(InRemoveCount == NumMips - InExtendedData->NumMipsInTail);

			// We want the layout for the entire tail.
			MipTextureDescription.NumMips = IntCastChecked<uint8>(InExtendedData->NumMipsInTail);
		}
		return MipTextureDescription;
	}
};

/**
*	Interface for platform formats that consume a linear, unpacked texture that an be built on
*	a host platform (e.g. windows) and then tile/pack it as necessary.
*/
class ITextureTiler
{
public:
	/**
	*	The generic texture tiling build function expects the following functions to exist that
	*	do what they say on the tin.
	* 
	*	static const FUtf8StringView GetBuildFunctionNameStatic()
	*	static FGuid GetBuildFunctionVersionGuid()
	*/
	
	/**
	* Generate and return any out-of-band data that needs to be saved for a given encoded texture description and LODBias.
	*/
	virtual FEncodedTextureExtendedData GetExtendedDataForTexture(const FEncodedTextureDescription& InTextureDescription, int8 InLODBias) const = 0;

	virtual const FUtf8StringView GetBuildFunctionName() const = 0;
	virtual const FUtf8StringView GetDetileBuildFunctionName() const = 0;

	/**
		InLinearSurfaces must have the necessary input mips for the mip level - i.e. for a packed mip tail,
		InMipIndex is the index of the top mip of the tail, and InLinearSurfaces must have all the source mips
		for the entire tail.
	*/
	virtual FSharedBuffer ProcessMipLevel(const FEncodedTextureDescription& InTextureDescription, const FEncodedTextureExtendedData& InExtendedData, TArrayView<FMemoryView> InLinearSurfaces, int32 InMipIndex) const = 0;
	
	/**
	*	Given a tiled mip chain, detile in to OutLinearMips. For mip tails, OutLinearMips.Num may end up larger than InTiledMips.Num.
	*	Mips have all slices concatenated together.
	*/
	virtual bool DetileMipChain(FEncodedTextureDescription::FUniqueBufferMipChain& OutLinearMips, FEncodedTextureDescription::FSharedBufferMipChain InTiledMips, const FEncodedTextureDescription& InTextureDescription, const FEncodedTextureExtendedData& InExtendedData, const FString& InTexturePathName) const
	{
		return false;
	}
};

/**
 * Interface for texture compression modules.
 * 
 * Note that if you add any virtual functions to this, they almost certainly need to be plumbed through
 * ChildTextureFormat! This is why the Format is passed around - ChildTextureFormat needs it to resolve to
 * the base format.
 */
class ITextureFormat
{
public:

	/**
	 * Checks whether this texture format can compress in parallel.
	 *
	 * @return true if parallel compression is supported, false otherwise.
	 */
	virtual bool AllowParallelBuild() const
	{
		return false;
	}

	/**
	*	Return the name of the encoder used for the given format.
	* 
	*	Used for debugging and UI.
	* */
	virtual FName GetEncoderName(FName Format) const = 0;

	/** 
		Exposes whether the format supports the fast/final encode speed switching in project settings. 
		Needs the Format so that we can thunk through the child texture formats correctly.
	*/
	virtual bool SupportsEncodeSpeed(FName Format, const ITargetPlatformSettings* TargetPlatform) const
	{
		return false;
	}

	/**
	 * @returns true in case Compress can handle other than RGBA32F image formats
	 */
	virtual bool CanAcceptNonF32Source(FName Format) const
	{
		return false;
	}

	// If the format can decode to RGBA8/RGBA16F, this is the IBuild function name for it.
	virtual const FUtf8StringView GetDecodeBuildFunctionName() const
	{
		return UTF8TEXTVIEW("DecodeUnsupported");
	}

	/**
	 * Gets the current version of the specified texture format.
	 *
	 * @param Format The format to get the version for.
	 * @return Version number.
	 */
	virtual uint16 GetVersion(
		FName Format,
		const FTextureBuildSettings* BuildSettings = nullptr
	) const = 0;

	/**
	 * Gets an optional derived data key string, so that the compressor can
	 * rely upon the number of mips, size of texture, etc, when compressing the image
	 *
	 * @param InBuildSettings Reference to the build settings we are compressing with.
	 * @param InMipCount Mip count of the physical texture that will be built - 0 for virtual textures.
 	 * @param InMip0Dimensions Mip width/height/slices of the physical texture that will be built - 0s for virtual textures.
	 * @return A string that will be used with the DDC, the string should be in the format "<DATA>_"
	 */
	virtual FString GetDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const
	{
		return TEXT("");
	}

	/**
	 * Gets the list of supported formats.
	 *
	 * @param OutFormats Will hold the list of formats.
	 */
	virtual void GetSupportedFormats( TArray<FName>& OutFormats ) const = 0;

	/**
	* Gets the capabilities of the texture compressor.
	*
	* @param OutCaps Filled with capability properties of texture format compressor.
	*/
	UE_DEPRECATED(5.1, "Hasn't been used in a while.")
	virtual FTextureFormatCompressorCaps GetFormatCapabilities() const { return FTextureFormatCompressorCaps(); }

	/**
	* Gets the capabilities of the texture compressor.
	*
	* @param OutCaps Filled with capability properties of texture format compressor.
	*/
	UE_DEPRECATED(5.1, "Use GetExtendedDataForTexture instead to get the same information without the actual image bits.")
	virtual FTextureFormatCompressorCaps GetFormatCapabilitiesEx(const FTextureBuildSettings& BuildSettings, uint32 NumMips, const struct FImage& ExampleImage, bool bImageHasAlphaChannel) const
	{
		return FTextureFormatCompressorCaps();
	}

	/**
	 * Calculate the final/runtime pixel format for this image on this platform
	 */
	UE_DEPRECATED(5.1, "Use GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel) instead")
	virtual EPixelFormat GetPixelFormatForImage(const FTextureBuildSettings& BuildSettings, const struct FImage& Image, bool bImageHasAlphaChannel) const
	{
		return GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel);
	}
	

	/**
	* Returns what the compressed pixel format will be for a given format and the given settings.
	* 
	* bInImageHasAlphaChannel is whether or not to treat the source image format as having an alpha channel,
	* independent of whether or not it actually has one.
	*/
	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& InBuildSettings, bool bInImageHasAlphaChannel) const
	{
		return PF_Unknown;
	}

	/**
	* Generate and return any out-of-band data that needs to be saved for a given encoded texture description. This is
	* for textures that have been transformed in some way for a platform. LODBias is needed because in some cases the tiling
	* changes based on the top mip actually given to the hardware.
	*/
	[[nodiscard]] virtual FEncodedTextureExtendedData GetExtendedDataForTexture(const FEncodedTextureDescription& InTextureDescription, int8 InLODBias) const
	{
		FEncodedTextureExtendedData ExtendedData;
		InTextureDescription.GetLinearMipSizesInBytes(ExtendedData.MipSizesInBytes);
		return ExtendedData;
	}

	// Return true if this format can decode the given pixel format to one of the ERawImageFormats.
	virtual bool CanDecodeFormat(EPixelFormat InPixelFormat) const { return false; }

	/**
	* Decodes an image encoded as a EPixelFormat into something encoded as a ERawImageFormat. This will only be called if CanDecodeFormat returns true.
	*/
	virtual bool DecodeImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, EPixelFormat InPixelFormat, bool bInSRGB, const FName& InTextureFormatName, FSharedBuffer InEncodedData, FImage& OutImage, FStringView InTextureName) const
	{
		return false;
	}

	/**
	 * Compresses a single image.
	 *
	 * @param Image The input image.  Image.RawData may be freed or modified by CompressImage; do not use after calling this.
	 * @param BuildSettings Build settings.
 	 * @param InMip0Dimensions X/Y = Width/Height; Z = 1 unless volume texture, then its depth
	 * @param InMip0NumSlicesNoDepth see FEncodedTextureDescription::NumSlices_NoDepth()
	 * @param InMipIndex Mip index of current image in the overall texture.
	 * @param InMipCount Total mips this texture will be created with.
	 * @param DebugTexturePathName The path name of the texture we are building, for debug logging/filtering/dumping.
	 * @param bImageHasAlphaChannel true if the image has a non-white alpha channel.
	 * @param OutCompressedMip The compressed image.
	 * @returns true on success, false otherwise.
	 */
	virtual bool CompressImage(
		const FImage& Image,
		const FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		struct FCompressedImage2D& OutCompressedImage
	) const = 0;

	/**
	 * Compress an image (or images for a miptail) into a single mip blob.
	 *
	 * @param Images The input image(s).  Image.RawData may be freed or modified by CompressImage; do not use after calling this.
	 * @param NumImages The number of images (for a miptail, this number should match what was returned in GetExtendedDataForTexture, mostly used for verification)
	 * @param BuildSettings Build settings.
	 * @param InMip0Dimensions X/Y = Width/Height; Z = 1 unless volume texture, then its depth
	 * @param InMip0NumSlicesNoDepth see FEncodedTextureDescription::NumSlices_NoDepth()
	 * @param InMipIndex Mip index of current image in the overall texture.
	 * @param InMipCount Total mips this texture will be created with.
	 * @param DebugTexturePathName The path name of the texture we are building, for debug logging/filtering/dumping.
	 * @param bImageHasAlphaChannel true if the image has a non-white alpha channel.
	 * @param ExtData Extra data that the format may want to have passed back in to each compress call (makes the format class be stateless)
	 * @param OutCompressedMip The compressed image.
	 * @returns true on success, false otherwise.
	 */
	virtual bool CompressImageEx(
		const FImage* Images,
		const uint32 NumImages,
		const FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		uint32 ExtData,
		FCompressedImage2D& OutCompressedImage) const
	{
		// general case can't handle mip tails
		if (Images == nullptr || NumImages > 1)
		{
			return false;
		}
		
		return CompressImage(*Images, BuildSettings, InMip0Dimensions, InMip0NumSlicesNoDepth, InMipIndex, InMipCount, DebugTexturePathName, bImageHasAlphaChannel, OutCompressedImage);
	}

	/**
	 * An object produced by PrepareTiling and used by SetTiling and CompressImageTiled.
	 * This is used as an inheritance base for tiling formats to add their own information.
	 */
	
	struct FTilerSettings
	{
	};

	/**
	 * Compress an image (or images for a miptail) into a single mip blob with device-specific tiling.
	 *
	 * @param Image The input image.  May be freed!
	 * @param BuildSettings Build settings.
	 * @param bImageHasAlphaChannel true if the image has a non-white alpha channel.
	 * @param DebugTexturePathName The path name of the texture we are building, for debug logging/filtering/dumping.
	 * @param OutCompressedMip The compressed image.
	 * @param Tiler The tiler settings.
	 * @returns true on success, false otherwise.
	 */
	UE_DEPRECATED(5.6, "Tiling compressors will no longer be supported; compress the mip entirely via CompressImage(Ex) and UE will tile the image afterwards")
	virtual bool CompressImageTiled(
		const FImage* Images,
		uint32 NumImages,
		const FTextureBuildSettings& BuildSettings,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		TSharedPtr<FTilerSettings>& TilerSettings,
		struct FCompressedImage2D& OutCompressedImage) const
	{
		unimplemented();
		return false;
	}

	/**
	 * Whether device-specific tiling is supported by the compressor.
	 *
	 * @param BuildSettings Build settings.
	 * @returns true if tiling is supported, false if it must be done by the caller
	 *
	 */
	UE_DEPRECATED(5.6, "Tiling compressors will no longer be supported; compress the mip entirely via CompressImage(Ex) and UE will tile the image afterwards")
	virtual bool SupportsTiling() const
	{
		return false;
	}

	/**
	 * Prepares to compresses a single image with tiling. The result OutTilerSettings is used by SetTiling and CompressImageTiled.
	 *
	 * @param Image The input image.
	 * @param BuildSettings Build settings.
	 * @param bImageHasAlphaChannel true if the image has a non-white alpha channel.
	 * @param OutTilerSettings The tiler settings that will be used by CompressImageTiled and SetTiling.
	 * @param OutCompressedImage The image to tile.
	 * @returns true on success, false otherwise.
	 */
	UE_DEPRECATED(5.6, "Tiling compressors will no longer be supported; compress the mip entirely via CompressImage(Ex) and UE will tile the image afterwards")
	virtual bool PrepareTiling(
		const FImage* Images,
		const uint32 NumImages,
		const FTextureBuildSettings& BuildSettings,
		bool bImageHasAlphaChannel,
		TSharedPtr<FTilerSettings>& OutTilerSettings,
		TArray<FCompressedImage2D>& OutCompressedImage
	) const
	{
		unimplemented();
		return false;
	}
	
	/**
	 * Sets the tiling settings after device-specific tiling has been performed.
	 *
	 * @param BuildSettings Build settings.
	 * @param TilerSettings The tiler settings produced by PrepareTiling.
	 * @param ReorderedBlocks The blocks that have been tiled.
	 * @param NumBlocks The number of blocks.
	 * @returns true on success, false otherwise.
	 */
	UE_DEPRECATED(5.6, "Tiling compressors will no longer be supported; compress the mip entirely via CompressImage(Ex) and UE will tile the image afterwards")
	virtual bool SetTiling(
		const FTextureBuildSettings& BuildSettings,
		TSharedPtr<FTilerSettings>& TilerSettings,
		const TArray64<uint8>& ReorderedBlocks,
		uint32 NumBlocks
	) const
	{
		unimplemented();
		return false;
	}

	/**
	 * Cleans up the FTilerSettings object once it is finished.
	 * 
	 * @param BuildSettings Build settings.
	 * @param TilerSettings The tiler settings object to release.
	 */
	UE_DEPRECATED(5.6, "Tiling compressors will no longer be supported; compress the mip entirely via CompressImage(Ex) and UE will tile the image afterwards")
	virtual void ReleaseTiling(const FTextureBuildSettings& BuildSettings, TSharedPtr<FTilerSettings>& TilerSettings) const
	{
		unimplemented();
	}

	/**
	 * Obtains the current global format config object for this texture format.
	 * 
	 * This is only ever called during task creation - never in a build worker
	 * (FormatConfigOverride is empty)
	 * 
	 * @param BuildSettings Build settings.
	 * @returns The current format config object or an empty object if no format config is defined for this texture format.
	 */
	virtual FCbObject ExportGlobalFormatConfig(const FTextureBuildSettings& BuildSettings) const
	{
		return FCbObject();
	}

	/**
	 * If this is an Alternate Texture Format, return the prefix to apply 
	 */
	virtual FString GetAlternateTextureFormatPrefix() const
	{
		return FString();
	}

	virtual const FChildTextureFormat* GetChildFormat() const
	{
		return nullptr;
	}
	
	/**
	 * Identify the latest sdk version for this texture encoder
	 *   (note the SdkVersion is different than the TextureFormat Version)
	 */
	virtual FName GetLatestSdkVersion() const
	{
		return FName();
	}
	
	UE_DEPRECATED(5.0, "Legacy API - do not use")
	virtual bool UsesTaskGraph() const
	{
		unimplemented();
		return true;
	}

public:

	/** Virtual destructor. */
	virtual ~ITextureFormat() = default;
};
