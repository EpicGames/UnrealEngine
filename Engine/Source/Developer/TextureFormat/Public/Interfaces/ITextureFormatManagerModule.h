// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Module for the texture format manager
 */
class ITextureFormatManagerModule
	: public IModuleInterface
{
public:

	/**
	 * Finds a texture format with the specified name. Safe to call from any thread.
	 *
	 * @param Name Name of the format to find.
	 * @return The texture format, or nullptr if not found.
	 */
	virtual const class ITextureFormat* FindTextureFormat( FName Name ) = 0;

	/**
	 * Finds a texture format with the specified name and provides information about the module it came from.
	 * Safe to call from any thread.
	 * 
	 * @param Name Name of the format to find.
	 * @param OutModuleName Name of the module that the found format came from, or unmodified if not found.
	 * @param OutModule Interface of the module that the found format came from, or unmodified if not found.
	 * @return The texture format, or nullptr if not found.
	 */
	virtual const class ITextureFormat* FindTextureFormatAndModule( FName Name, FName& OutModuleName, class ITextureFormatModule*& OutModule ) = 0;

	/**
	 * Returns the list of all ITextureFormats that were located in DLLs.
	 *
	 * @return Collection of texture formats.
	 */
	virtual TArray<const class ITextureFormat*> GetAllTextureFormats() = 0;

	/**
	 * Invalidates the texture format manager module.
	 * 
	 * This is no longer necessary as all work is done in response to broadcast plugin/module discovery messages.
	 */
	UE_DEPRECATED(5.6, "No longer necessary to call")
	void Invalidate() { }

public:

	/** Virtual destructor. */
	~ITextureFormatManagerModule() = default;
};
