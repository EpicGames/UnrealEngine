// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class ITextureFormat;

/**
 * Interface for texture format modules.
 */
class ITextureFormatModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the texture format.
	 *
	 * @return The texture format interface.
	 */
	virtual ITextureFormat* GetTextureFormat() = 0;

	/**
	* Will this TextureFormat call back to Managermodule GetTextureFormats ?
	* 
	*/
	virtual bool CanCallGetTextureFormats() = 0;

public:

	/** Virtual destructor. */
	~ITextureFormatModule() = default;
};
