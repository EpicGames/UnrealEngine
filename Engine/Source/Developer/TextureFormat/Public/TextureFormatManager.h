// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Modules/ModuleManager.h"

/** Return the Texture Format Manager interface, if it is available, otherwise return nullptr. **/
inline ITextureFormatManagerModule* GetTextureFormatManager()
{
	// GetModule , not Load.  Should be loaded by TargetPlatform or Program startup on main thread.
	//  cannot load modules from threads so must be done before any thread needs this module
	ITextureFormatManagerModule* TFMM = FModuleManager::GetModulePtr<ITextureFormatManagerModule>("TextureFormat");
	if ( TFMM == nullptr )
	{
		UE_LOGF(LogInit, Error, "Texture format manager Ptr was requested but not available.");
	}
	return TFMM;
}

/** Return the Texture Format Manager interface, fatal error if it is not available. **/
inline ITextureFormatManagerModule& GetTextureFormatManagerRef()
{
	class ITextureFormatManagerModule* TextureFormatManager = GetTextureFormatManager();
	if (!TextureFormatManager)
	{
		UE_LOGF(LogInit, Fatal, "Texture format manager Ref was requested, but not available.");
		CA_ASSUME( TextureFormatManager != NULL );	// Suppress static analysis warning in unreachable code (fatal error)
	}
	return *TextureFormatManager;
}

