// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormat : ModuleRules
{
	public TextureFormat(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"TextureCompressor"
		});

		//AddEngineThirdPartyPrivateStaticDependencies(Target, "nvTextureTools");
	}
}
